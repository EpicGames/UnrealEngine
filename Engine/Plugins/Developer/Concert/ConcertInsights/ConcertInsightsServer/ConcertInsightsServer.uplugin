{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ConcertInsightsServer",
	"Description": "Listens for requests of clients to start synchronized tracing, which is initiated in the ConcertInsightsClient plugin.",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"Hidden": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"SupportedPrograms": [
		"UnrealMultiUserServer",
		"UnrealMultiUserSlateServer"
	],
	"Modules": [
		{
			"Name": "ConcertInsightsServer",
			"Type": "Program",
			"LoadingPhase": "Default",
			"ProgramAllowList": [
				"UnrealMultiUserServer",
				"UnrealMultiUserSlateServer" 
			]
		}
	],
	"Plugins": [
		{
			"Name": "ConcertInsightsCore",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncServer",
			"Enabled": true
		}
	]
}