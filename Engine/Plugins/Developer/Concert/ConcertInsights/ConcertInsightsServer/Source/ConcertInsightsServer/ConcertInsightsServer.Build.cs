// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ConcertInsightsServer : ModuleRules
{
    public ConcertInsightsServer(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "ConcertInsightsCore", 
	            "ConcertSyncServer", 
                "CoreUObject",
                "Slate",
                "SlateCore",
                "ToolMenus", 
            }
        );
        
        ShortName = "CrtInS";
    }
}