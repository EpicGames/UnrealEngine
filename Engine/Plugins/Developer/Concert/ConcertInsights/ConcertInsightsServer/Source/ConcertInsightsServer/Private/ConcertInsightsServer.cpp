// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertInsightsServer.h"

#define LOCTEXT_NAMESPACE "FConcertInsightsServerModule"

namespace UE::ConcertInsightsServer
{
	void FConcertInsightsServerModule::StartupModule()
	{
		TraceControls = ConcertInsightsCore::FTraceControls::Make<FServerTraceControls>();
	}

	void FConcertInsightsServerModule::ShutdownModule()
	{}
}


#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(UE::ConcertInsightsServer::FConcertInsightsServerModule, ConcertInsightsServer)