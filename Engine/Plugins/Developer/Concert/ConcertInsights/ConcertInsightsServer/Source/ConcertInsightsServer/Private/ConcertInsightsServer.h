// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertInsightsServerModule.h"
#include "ServerTraceControls.h"
#include "Templates/UniquePtr.h"

namespace UE::ConcertInsightsServer
{
	class FConcertInsightsServerModule : public IConcertInsightsServerModule
	{
	public:
		
		//~ Begin IModuleInterface Interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface Interface

		FServerTraceControls& GetTraceControls() { return *TraceControls; }

	private:

		/** The local state of synchronized tracing. */
		TUniquePtr<FServerTraceControls> TraceControls;
	};
}


