// Copyright Epic Games, Inc. All Rights Reserved.

#include "ServerTraceControls.h"

#include "IConcertServer.h"
#include "IConcertSyncServer.h"
#include "IConcertSyncServerModule.h"

namespace UE::ConcertInsightsServer
{
	FServerTraceControls::FServerTraceControls()
	{
		IConcertSyncServerModule::Get().OnServerCreated().AddRaw(this, &FServerTraceControls::OnServerCreated);
	}

	FServerTraceControls::~FServerTraceControls()
	{
		if (IConcertSyncServerModule::IsAvailable())
		{
			IConcertSyncServerModule::Get().OnServerCreated().RemoveAll(this);
		}

		const TSharedPtr<IConcertServerSession> ServerSession = InProgressSynchronizedServerTrace ? InProgressSynchronizedServerTrace->InitiatingSession.Pin() : nullptr;
		if (ServerSession)
		{
			ServerSession->OnSessionClientChanged().RemoveAll(this);
		}

		// ~FTraceControls will clean up delegates of registered sessions itself
	}

	void FServerTraceControls::OnSynchronizedTraceAccepted(const FConcertSessionContext& Context, const FConcertTrace_StartSyncTrace_Request& Request, const TSharedRef<IConcertSession>& Session)
	{
		// This is the equivalent of calling ServerInstance.Pin()->GetConcertServer()->GetLiveSession(Session->GetId())
		TSharedRef<IConcertServerSession> ServerSession = StaticCastSharedRef<IConcertServerSession>(Session);
		
		check(!InProgressSynchronizedServerTrace.IsSet());
		InProgressSynchronizedServerTrace.Emplace(Context.SourceEndpointId, ServerSession);
		
		ServerSession->OnSessionClientChanged().AddRaw(this, &FServerTraceControls::OnSynchronizedTraceClientChanged);
		OnSynchronizedTraceStopped().AddRaw(this, &FServerTraceControls::CleanUpClientsChangedDelegate);
	}

	bool FServerTraceControls::CanSendRequestsToEndpoint(const FGuid& EndpointId, const IConcertSession& Session) const
	{
		return Session.GetSessionInfo().ServerEndpointId != EndpointId;
	}

	ConcertInsightsCore::FInitArgs FServerTraceControls::GetInitEventArgs() const
	{
		return { {}, TEXT("Server"), true };
	}

	void FServerTraceControls::OnServerCreated(TWeakPtr<IConcertSyncServer> Server)
	{
		if (!ensureMsgf(!ServerInstance.IsValid(), TEXT("We assume there is only one server instance per application.")))
		{
			return;
		}
		
		if (const TSharedPtr<IConcertSyncServer> ServerPin = Server.Pin())
		{
			ServerInstance = Server;
			RegisterHandlersForSessions(*ServerPin->GetConcertServer());
		}
	}

	void FServerTraceControls::RegisterHandlersForSessions(IConcertServer& Server)
	{
		Server.OnConcertServerSessionStartup().AddRaw(this, &FServerTraceControls::OnSessionStartup);
		for (const TSharedPtr<IConcertServerSession>& Session : Server.GetLiveSessions())
		{
			RegisterTraceRequestsHandler(Session.ToSharedRef());
		}
	}

	void FServerTraceControls::OnSessionStartup(TWeakPtr<IConcertServerSession> Session)
	{
		if (const TSharedPtr<IConcertServerSession> SessionPin = Session.Pin())
		{
			RegisterTraceRequestsHandler(SessionPin.ToSharedRef());
		}
	}

	void FServerTraceControls::OnSynchronizedTraceClientChanged(IConcertServerSession& Session, EConcertClientStatus Status, const FConcertSessionClientInfo& ClientInfo)
	{
		if (Status == EConcertClientStatus::Disconnected
			&& InProgressSynchronizedServerTrace->SynchronizedTraceInstigator == ClientInfo.ClientEndpointId)
		{
			StopSynchronizedTrace();
			CleanUpClientsChangedDelegate();
		}
	}

	void FServerTraceControls::CleanUpClientsChangedDelegate()
	{
		if (const TSharedPtr<IConcertServerSession> ServerSession = InProgressSynchronizedServerTrace ? InProgressSynchronizedServerTrace->InitiatingSession.Pin() : nullptr)
		{
			ServerSession->OnSessionClientChanged().RemoveAll(this);
		}

		InProgressSynchronizedServerTrace.Reset();
	}
}
