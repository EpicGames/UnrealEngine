// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceControls.h"

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"

class IConcertServer;
class IConcertServerSession;
class IConcertSyncServer;

namespace UE::ConcertInsightsServer
{
	/**
	 * Server controls for the editor.
	 *
	 * This class also keeps track of the client that instigated the synchronized trace.
	 * If the instigating client disconnects, all other endpoints are told to stop tracing.
	 */
	class FServerTraceControls : public ConcertInsightsCore::FTraceControls
	{
		friend class FTraceControls; // For FTraceControls::Make
	public:

		virtual ~FServerTraceControls() override;
		
	protected:
		
		//~ Begin FTraceControls Interface
		virtual void OnSynchronizedTraceAccepted(const FConcertSessionContext& Context, const FConcertTrace_StartSyncTrace_Request& Request, const TSharedRef<IConcertSession>& Session) override;
		virtual bool CanSendRequestsToEndpoint(const FGuid& EndpointId, const IConcertSession& Session) const override;
		virtual ConcertInsightsCore::FInitArgs GetInitEventArgs() const override;
		//~ End FTraceControls Interface
		
	private:

		struct FSynchronizedSessionServerData
		{
			/**
			 * Endpoints of the endpoint that instigated the synchronized trace.
			 * If this endpoint disconnects, the server tells all other clients to terminate the synchronized trace.
			 */
			FGuid SynchronizedTraceInstigator;

			/** The server session on which this trace was started. */
			TWeakPtr<IConcertServerSession> InitiatingSession;
		};

		/** Keeps track of the created server. Usually has 0 or 1 entries. */
		TWeakPtr<IConcertSyncServer> ServerInstance;
		
		/** Additional data for the server about the synchronized trace. */
		TOptional<FSynchronizedSessionServerData> InProgressSynchronizedServerTrace;
		
		FServerTraceControls();
		
		void OnServerCreated(TWeakPtr<IConcertSyncServer> Server);
		void RegisterHandlersForSessions(IConcertServer& Server);
		
		void OnSessionStartup(TWeakPtr<IConcertServerSession> Session);
		void OnSynchronizedTraceClientChanged(IConcertServerSession& ConcertServerSession, EConcertClientStatus ConcertClientStatus, const FConcertSessionClientInfo& ConcertSessionClientInfo);
		
		void CleanUpClientsChangedDelegate();
	};
}
